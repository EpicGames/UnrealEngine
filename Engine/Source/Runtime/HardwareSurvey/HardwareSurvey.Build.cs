// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class HardwareSurvey : ModuleRules
{
	public HardwareSurvey(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"ApplicationCore"
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
		new string[] {
				"Analytics",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Analytics",
			}
		);

		PrecompileForTargets = PrecompileTargetsType.Any;
	}
}
