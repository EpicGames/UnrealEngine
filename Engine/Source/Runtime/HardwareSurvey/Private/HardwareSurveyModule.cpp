// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformSurvey.h"
#include "HAL/PlatformSurvey.h"
#include "Containers/Ticker.h"
#include "Modules/ModuleManager.h"
#include "IHardwareSurveyModule.h"
#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "Stats/Stats.h"

/**
 * Implements the HardwareSurvey module.
 */
class FHardwareSurveyModule
	: public IHardwareSurveyModule
{

public:

	// IModuleInterface interface

	virtual void StartupModule() override
	{
		Analytics = nullptr;
		bPendingHardwareSurveyResults = false;
	}

	virtual void ShutdownModule() override
	{
		if (bPendingHardwareSurveyResults)
		{
			FTSTicker::RemoveTicker(TickerHandle);
			bPendingHardwareSurveyResults = false;
		}
	}

public:

	// IHardwareSurveyModule interface

	virtual void StartHardwareSurvey(IAnalyticsProvider& AnalyticsProvider) override
	{
		// Bail if we already waiting on a hardware survey
		if (bPendingHardwareSurveyResults)
		{
			return;
		}

		Analytics = &AnalyticsProvider;

		if (IsHardwareSurveyRequired())
		{
			bPendingHardwareSurveyResults = true;
			TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FHardwareSurveyModule::TickHardwareSurvey));
		}
	}

protected:


	bool TickHardwareSurvey(float Delta)
	{
        QUICK_SCOPE_CYCLE_COUNTER(STAT_HardwareSurveyModule_TickHardwareSurvey);

		bool bContinueTick = true;
		if (bPendingHardwareSurveyResults)
		{
			FHardwareSurveyResults HardwareSurveyResults;
			if (FPlatformSurvey::GetSurveyResults(HardwareSurveyResults))
			{
				OnHardwareSurveyComplete(HardwareSurveyResults);
				bPendingHardwareSurveyResults = false;
				bContinueTick = false;
			}
		}
		else
		{
			bContinueTick = false;
		}
		return bContinueTick;
	}

	bool IsHardwareSurveyRequired()
	{
		// Analytics must have been initialized FIRST.
		if (Analytics == nullptr || IsRunningDedicatedServer() || GIsBuildMachine)
		{
			return false;
		}

#if PLATFORM_IOS || PLATFORM_ANDROID || PLATFORM_DESKTOP
		bool bSurveyDone = false;
		bool bSurveyExpired = false;

		// platform agnostic code to get the last time we did a survey
		FString LastRecordedTimeString;
		if (FPlatformMisc::GetStoredValue(TEXT("Epic Games"), TEXT("Unreal Engine/Hardware Survey"), TEXT("HardwareSurveyDateTime"), LastRecordedTimeString))
		{
			// attempt to convert to FDateTime
			FDateTime LastRecordedTime;
			if (FDateTime::Parse(LastRecordedTimeString, LastRecordedTime))
			{
				bSurveyDone = true;

				// make sure it was a month ago
				FTimespan Diff = FDateTime::UtcNow() - LastRecordedTime;

				if (Diff.GetTotalDays() > 30)
				{
					bSurveyExpired = true;
				}
			}
		}

		return !bSurveyDone || bSurveyExpired;
#else
		return false;
#endif
	}

	FString HardwareSurveyBucketRAM(uint32 MemoryMB)
	{
		const float GBToMB = 1024.0f;
		FString BucketedRAM;

		if (MemoryMB < 2.0f * GBToMB) BucketedRAM = TEXT("<2GB");
		else if (MemoryMB < 4.0f * GBToMB) BucketedRAM = TEXT("2GB-4GB");
		else if (MemoryMB < 6.0f * GBToMB) BucketedRAM = TEXT("4GB-6GB");
		else if (MemoryMB < 8.0f * GBToMB) BucketedRAM = TEXT("6GB-8GB");
		else if (MemoryMB < 12.0f * GBToMB) BucketedRAM = TEXT("8GB-12GB");
		else if (MemoryMB < 16.0f * GBToMB) BucketedRAM = TEXT("12GB-16GB");
		else if (MemoryMB < 20.0f * GBToMB) BucketedRAM = TEXT("16GB-20GB");
		else if (MemoryMB < 24.0f * GBToMB) BucketedRAM = TEXT("20GB-24GB");
		else if (MemoryMB < 28.0f * GBToMB) BucketedRAM = TEXT("24GB-28GB");
		else if (MemoryMB < 32.0f * GBToMB) BucketedRAM = TEXT("28GB-32GB");
		else if (MemoryMB < 36.0f * GBToMB) BucketedRAM = TEXT("32GB-36GB");
		else BucketedRAM = TEXT(">36GB");

		return MoveTemp(BucketedRAM);
	}

	FString HardwareSurveyBucketVRAM(uint32 VidMemoryMB)
	{
		const float GBToMB = 1024.0f;
		FString BucketedVRAM;

		if (VidMemoryMB < 0.25f * GBToMB) BucketedVRAM = TEXT("<256MB");
		else if (VidMemoryMB < 0.5f * GBToMB) BucketedVRAM = TEXT("256MB-512MB");
		else if (VidMemoryMB < 1.0f * GBToMB) BucketedVRAM = TEXT("512MB-1GB");
		else if (VidMemoryMB < 1.5f * GBToMB) BucketedVRAM = TEXT("1GB-1.5GB");
		else if (VidMemoryMB < 2.0f * GBToMB) BucketedVRAM = TEXT("1.5GB-2GB");
		else if (VidMemoryMB < 2.5f * GBToMB) BucketedVRAM = TEXT("2GB-2.5GB");
		else if (VidMemoryMB < 3.0f * GBToMB) BucketedVRAM = TEXT("2.5GB-3GB");
		else if (VidMemoryMB < 4.0f * GBToMB) BucketedVRAM = TEXT("3GB-4GB");
		else if (VidMemoryMB < 6.0f * GBToMB) BucketedVRAM = TEXT("4GB-6GB");
		else if (VidMemoryMB < 8.0f * GBToMB) BucketedVRAM = TEXT("6GB-8GB");
		else BucketedVRAM = TEXT(">8GB");

		return MoveTemp(BucketedVRAM);
	}

	FString HardwareSurveyBucketResolution(uint32 DisplayWidth, uint32 DisplayHeight)
	{
		FString BucketedRes;
		float AspectRatio = (float)DisplayWidth / DisplayHeight;

		if (AspectRatio < 1.5f)
		{
			// approx 4:3
			if (DisplayWidth < 1150)
			{
				BucketedRes = TEXT("1024x768");
			}
			else if (DisplayHeight < 912)
			{
				BucketedRes = TEXT("1280x800");
			}
			else
			{
				BucketedRes = TEXT("1280x1024");
			}
		}
		else
		{
			// widescreen
			if (DisplayWidth < 1400)
			{
				BucketedRes = TEXT("1366x768");
			}
			else if (DisplayWidth < 1520)
			{
				BucketedRes = TEXT("1440x900");
			}
			else if (DisplayWidth < 1640)
			{
				BucketedRes = TEXT("1600x900");
			}
			else if (DisplayWidth < 1800)
			{
				BucketedRes = TEXT("1680x1050");
			}
			else if (DisplayHeight < 1140)
			{
				BucketedRes = TEXT("1920x1080");
			}
			else
			{
				BucketedRes = TEXT("1920x1200");
			}
		}

		return MoveTemp(BucketedRes);
	}

	FString HardwareSurveyGetResolutionClass(uint32 LargestDisplayHeight)
	{
		FString ResolutionClass = "720";

		if (LargestDisplayHeight < 700)
		{
			ResolutionClass = TEXT("<720");
		}
		else if (LargestDisplayHeight > 1024)
		{
			ResolutionClass = TEXT("1080+");
		}

		return MoveTemp(ResolutionClass);
	}

	void OnHardwareSurveyComplete(const FHardwareSurveyResults& SurveyResults)
	{
#if PLATFORM_IOS || PLATFORM_ANDROID || PLATFORM_DESKTOP
		if (Analytics != nullptr)
		{
			// remember last time we did a survey
			FPlatformMisc::SetStoredValue(TEXT("Epic Games"), TEXT("Unreal Engine/Hardware Survey"), TEXT("HardwareSurveyDateTime"), FDateTime::UtcNow().ToString());

#if PLATFORM_IOS || PLATFORM_ANDROID

			TArray<FAnalyticsEventAttribute> HardwareStatsAttribs;
			// copy from what IOSPlatformSurvey has filled out
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("Model"), SurveyResults.Platform));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("OS.Version"), SurveyResults.OSVersion));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("OS.Bits"), FString::Printf(TEXT("%d-bit"), SurveyResults.OSBits)));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("OS.Language"), SurveyResults.OSLanguage));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("RenderingAPI"), SurveyResults.RenderingAPI));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("CPU.Count"), FString::Printf(TEXT("%d"), SurveyResults.CPUCount)));
			FString DisplayResolution = FString::Printf(TEXT("%dx%d"), SurveyResults.Displays[0].CurrentModeWidth, SurveyResults.Displays[0].CurrentModeHeight);
			FString ViewResolution = FString::Printf(TEXT("%dx%d"), SurveyResults.Displays[0].CurrentModeWidth, SurveyResults.Displays[0].CurrentModeHeight);
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("DisplayResolution"), DisplayResolution));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("ViewResolution"), ViewResolution));

#if PLATFORM_ANDROID
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("GPUModel"), SurveyResults.RHIAdapter.AdapterName));
#endif

			Analytics->RecordEvent(*FString::Printf(TEXT("%hsHardwareStats"), FPlatformProperties::IniPlatformName()), HardwareStatsAttribs);

#elif PLATFORM_DESKTOP

			TArray<FAnalyticsEventAttribute> HardwareWEIAttribs;
			HardwareWEIAttribs.Add(FAnalyticsEventAttribute(TEXT("CPU.WEI"), FString::Printf(TEXT("%.1f"), SurveyResults.CPUPerformanceIndex)));
			HardwareWEIAttribs.Add(FAnalyticsEventAttribute(TEXT("GPU.WEI"), FString::Printf(TEXT("%.1f"), SurveyResults.GPUPerformanceIndex)));
			HardwareWEIAttribs.Add(FAnalyticsEventAttribute(TEXT("Memory.WEI"), FString::Printf(TEXT("%.1f"), SurveyResults.RAMPerformanceIndex)));

			Analytics->RecordEvent(TEXT("Hardware.WEI.1"), HardwareWEIAttribs);

			FString MainGPUName("Unknown");
			float MainGPUVRAMMB = 0.0f;
			FString MainGPUDriverVer("UnknownVersion");
			if (SurveyResults.DisplayCount > 0)
			{
				MainGPUName = SurveyResults.RHIAdapter.AdapterName;
				FString strAdapterDedicatedMemoryMB(SurveyResults.RHIAdapter.AdapterDedicatedMemoryMB);
				MainGPUVRAMMB = FCString::Atof(*strAdapterDedicatedMemoryMB);
				MainGPUDriverVer = SurveyResults.RHIAdapter.AdapterUserDriverVersion;
			}

			uint32 LargestDisplayHeight = 0;
			FString DisplaySize[3];
			if (SurveyResults.DisplayCount > 0)
			{
				DisplaySize[0] = HardwareSurveyBucketResolution(SurveyResults.Displays[0].CurrentModeWidth, SurveyResults.Displays[0].CurrentModeHeight);
				LargestDisplayHeight = FMath::Max(LargestDisplayHeight, SurveyResults.Displays[0].CurrentModeHeight);
			}
			if (SurveyResults.DisplayCount > 1)
			{
				DisplaySize[1] = HardwareSurveyBucketResolution(SurveyResults.Displays[1].CurrentModeWidth, SurveyResults.Displays[1].CurrentModeHeight);
				LargestDisplayHeight = FMath::Max(LargestDisplayHeight, SurveyResults.Displays[1].CurrentModeHeight);
			}
			if (SurveyResults.DisplayCount > 2)
			{
				DisplaySize[2] = HardwareSurveyBucketResolution(SurveyResults.Displays[2].CurrentModeWidth, SurveyResults.Displays[2].CurrentModeHeight);
				LargestDisplayHeight = FMath::Max(LargestDisplayHeight, SurveyResults.Displays[2].CurrentModeHeight);
			}

			// Resolution Class
			FString ResolutionClass;
			if (LargestDisplayHeight < 700)
			{
				ResolutionClass = TEXT("<720");
			}
			else if (LargestDisplayHeight < 1024)
			{
				ResolutionClass = TEXT("720");
			}
			else
			{
				ResolutionClass = TEXT("1080+");
			}

			// Bucket RAM
			FString BucketedRAM = HardwareSurveyBucketRAM(SurveyResults.MemoryMB);

			// Bucket VRAM
			FString BucketedVRAM = HardwareSurveyBucketVRAM(MainGPUVRAMMB);

			TArray<FAnalyticsEventAttribute> HardwareStatsAttribs;
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("Platform"), SurveyResults.Platform));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("CPU.WEI"), FString::Printf(TEXT("%.1f"), SurveyResults.CPUPerformanceIndex)));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("CPU.Brand"), SurveyResults.CPUBrand));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("CPU.Speed"), FString::Printf(TEXT("%.1fGHz"), SurveyResults.CPUClockGHz)));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("CPU.Count"), FString::Printf(TEXT("%d"), SurveyResults.CPUCount)));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("CPU.Name"), SurveyResults.CPUNameString));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("CPU.Info"), FString::Printf(TEXT("0x%08x"), SurveyResults.CPUInfo)));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("GPU.WEI"), FString::Printf(TEXT("%.1f"), SurveyResults.GPUPerformanceIndex)));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("GPU.Name"), MainGPUName));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("GPU.VRAM"), BucketedVRAM));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("GPU.DriverVersion"), MainGPUDriverVer));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("GPU.RHIAdapterName"), SurveyResults.RHIAdapter.AdapterName));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("GPU.RHIAdapterInternalDriverVersion"), SurveyResults.RHIAdapter.AdapterInternalDriverVersion));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("GPU.RHIAdapterUserDriverVersion"), SurveyResults.RHIAdapter.AdapterUserDriverVersion));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("GPU.RHIAdapterDriverDate"), SurveyResults.RHIAdapter.AdapterDriverDate));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("RAM"), BucketedRAM));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("RAM.WEI"), FString::Printf(TEXT("%.1f"), SurveyResults.RAMPerformanceIndex)));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("NumberOfMonitors"), FString::Printf(TEXT("%d"), SurveyResults.DisplayCount)));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("MonitorResolution.0"), DisplaySize[0]));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("MonitorResolution.1"), DisplaySize[1]));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("MonitorResolution.2"), DisplaySize[2]));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("ResolutionClass"), ResolutionClass));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("OS.Version"), SurveyResults.OSVersion));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("OS.SubVersion"), SurveyResults.OSSubVersion));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("OS.Bits"), FString::Printf(TEXT("%d-bit"), SurveyResults.OSBits)));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("OS.Language"), SurveyResults.OSLanguage));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("IsLaptop"), SurveyResults.bIsLaptopComputer ? TEXT("true") : TEXT("false")));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("IsRemoteSession"), SurveyResults.bIsRemoteSession ? TEXT("true") : TEXT("false")));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("SynthIdx.CPU0"), FString::Printf(TEXT("%.1f"), SurveyResults.SynthBenchmark.CPUStats[0].ComputePerfIndex())));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("SynthIdx.CPU1"), FString::Printf(TEXT("%.1f"), SurveyResults.SynthBenchmark.CPUStats[1].ComputePerfIndex())));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("SynthIdx.GPU0"), FString::Printf(TEXT("%.1f"), SurveyResults.SynthBenchmark.GPUStats[0].ComputePerfIndex())));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("SynthIdx.GPU1"), FString::Printf(TEXT("%.1f"), SurveyResults.SynthBenchmark.GPUStats[1].ComputePerfIndex())));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("SynthIdx.GPU2"), FString::Printf(TEXT("%.1f"), SurveyResults.SynthBenchmark.GPUStats[2].ComputePerfIndex())));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("SynthIdx.GPU3"), FString::Printf(TEXT("%.1f"), SurveyResults.SynthBenchmark.GPUStats[3].ComputePerfIndex())));
			HardwareStatsAttribs.Add(FAnalyticsEventAttribute(TEXT("SynthIdx.GPU4"), FString::Printf(TEXT("%.1f"), SurveyResults.SynthBenchmark.GPUStats[4].ComputePerfIndex())));

			Analytics->RecordEvent(TEXT("HardwareStats.1"), HardwareStatsAttribs);

			TArray<FAnalyticsEventAttribute> HardwareStatErrorsAttribs;
			HardwareStatErrorsAttribs.Add(FAnalyticsEventAttribute(TEXT("ErrorCount"), FString::Printf(TEXT("%d"), SurveyResults.ErrorCount)));
			HardwareStatErrorsAttribs.Add(FAnalyticsEventAttribute(TEXT("LastError"), SurveyResults.LastSurveyError));
			HardwareStatErrorsAttribs.Add(FAnalyticsEventAttribute(TEXT("LastError.Detail"), SurveyResults.LastSurveyErrorDetail));
			HardwareStatErrorsAttribs.Add(FAnalyticsEventAttribute(TEXT("LastError.WEI"), SurveyResults.LastPerformanceIndexError));
			HardwareStatErrorsAttribs.Add(FAnalyticsEventAttribute(TEXT("LastError.WEI.Detail"), SurveyResults.LastPerformanceIndexErrorDetail));

			Analytics->RecordEvent(TEXT("HardwareStatErrors.1"), HardwareStatErrorsAttribs);
#endif // PLATFORM_DESKTOP
		}
#endif
	}


private:
	// If true, the engine tick function will poll FPlatformSurvey for results
	bool bPendingHardwareSurveyResults;

	// Holds the analytics provider if available
	IAnalyticsProvider* Analytics;

	FTSTicker::FDelegateHandle TickerHandle;
};


IMPLEMENT_MODULE( FHardwareSurveyModule, HardwareSurvey );
