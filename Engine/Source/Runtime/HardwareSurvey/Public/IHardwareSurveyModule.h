// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class IAnalyticsProvider;

/**
 * Interface for the hardware survey module.
 */
class IHardwareSurveyModule
	: public IModuleInterface
{
public:

	/**
	 * Singleton-like access to this module's interface.
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IHardwareSurveyModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IHardwareSurveyModule>("HardwareSurvey");
	}

	/**
	 * Checks to see if this module is loaded and ready.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("HardwareSurvey");
	}

	/**
	 * Init and begin the async platform hardware survey.
	 *
	 * @param AnalyticsProvider		The analytics provider to use when sending survey info.
	 */
	virtual void StartHardwareSurvey(IAnalyticsProvider& AnalyticsProvider) = 0;


public:

	/** Virtual destructor. */
	virtual ~IHardwareSurveyModule() { }
};
